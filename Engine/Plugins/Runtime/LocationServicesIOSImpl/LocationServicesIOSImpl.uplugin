{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mobile Location Services - IOS Implementation",
	"Description": "IOS implementation for blueprint access for location data from mobile devices",
	"Category": "Mobile",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "LocationServicesIOSEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LocationServicesIOSImpl",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"IOS"
			]
		}
	],
	"Plugins": [
		{
			"Name": "LocationServicesBPLibrary",
			"Enabled": true
		}
	],
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"IsExperimentalVersion": false
}