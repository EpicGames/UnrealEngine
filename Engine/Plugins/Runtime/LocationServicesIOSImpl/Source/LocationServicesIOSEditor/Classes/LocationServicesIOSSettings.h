// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "LocationServicesIOSSettings.generated.h"

UCLASS(config = Engine, defaultconfig)
class ULocationServicesIOSSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()
	
	/* Text to display when requesting permission for Location Services */
    UPROPERTY(config, EditAnywhere, Category = LocationServices, meta = (DisplayName = "Location Services Always Use Permission Text" ))
    FString LocationAlwaysUsageDescription;
    
    /* Text to display when requesting permission for Location Services */
    UPROPERTY(config, EditAnywhere, Category = LocationServices, meta = (DisplayName = "Location Services In-Use Permission Text" ))
    FString LocationWhenInUseDescription;
};
