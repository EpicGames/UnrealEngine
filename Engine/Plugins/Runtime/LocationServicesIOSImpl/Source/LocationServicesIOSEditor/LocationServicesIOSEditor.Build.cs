// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LocationServicesIOSEditor : ModuleRules
{
    public LocationServicesIOSEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] 
			{
				"Core",
				"CoreUObject",
                "Engine",
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] 
				{
					"Settings",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] 
				{
					"Settings",
				}
			);
		}
	}
}
