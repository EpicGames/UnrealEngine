// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocationServicesIOSEditor.h"
#include "LocationServicesIOSSettings.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#endif


IMPLEMENT_MODULE( FLocationServicesIOSEditorModule, LocationServicesIOSEditor );

#define LOCTEXT_NAMESPACE "FLocationServicesIOSEditorModule"

void FLocationServicesIOSEditorModule::StartupModule()
{
#if WITH_EDITOR
	// register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "Location Services IOS",
			LOCTEXT("LocationServicesIOSSettingsName", "Location Services - IOS"),
			LOCTEXT("LocationServicesIOSSettingsDescription", "Configure the Location Services settings for IOS"),
			GetMutableDefault<ULocationServicesIOSSettings>()
		);
	}
#endif // WITH_EDITOR
}

void FLocationServicesIOSEditorModule::ShutdownModule()
{
#if WITH_EDITOR
	// unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Location Services IOS");
	}
#endif
}

ULocationServicesIOSSettings::ULocationServicesIOSSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#undef LOCTEXT_NAMESPACE
