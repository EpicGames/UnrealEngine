// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LocationServicesImpl.h"

#include "LocationServicesIOSImpl.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLocationServicesIOS, Log, All);

@class FLocationManagerDelegate;

UCLASS()
class ULocationServicesIOSImpl : public ULocationServicesImpl
{
	GENERATED_BODY()

public:
	
	virtual ~ULocationServicesIOSImpl();

	/**
	* Called to set up the Location Service before use
	*
	* @param Accuracy - as seen in the enum above
	* @param UpdateFrequency - in milliseconds. (Android only)
	* @param MinDistance - minDistance before a location update, in meters. 0 here means "update asap"
	* @return - true if Initialization was succesful
	*/
	virtual bool InitLocationServices(ELocationAccuracy Accuracy, float UpdateFrequency, float MinDistance) override;

	/**
	* Starts requesting location updates from the appropriate Location Service
	* @return - true if startup successful
	*/
	virtual bool StartLocationService() override;

	/**
	* Stops the updates of location from the Location Service that was started with StartLocationService
	* @return - true if stop is successful
	*/
	virtual bool StopLocationService() override;

	/**
	* Returns the last location information returned by the location service. If no location update has been made, will return
	* a default-value-filled struct.
	* @return - the last known location from updates
	*/
	virtual FLocationServicesData GetLastKnownLocation() override;

	/**
	* Checks if the supplied Accuracy is available on the current device.
	* @param Accuracy - the accuracy to check
	* @return - true if the mobile device can support the Accuracy, false if it will use a different accuracy
	*/
	virtual bool IsLocationAccuracyAvailable(ELocationAccuracy Accuracy) override;
	
	/**
	* Checks if the Location Services on the mobile device are enabled for this application
	* @return - true if the mobile device has enabled the appropriate service for the app
	*/
	virtual bool IsLocationServiceEnabled() override;

private:

    FLocationManagerDelegate *LocationDelegate;

};
