// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class LocationServicesIOSImpl : ModuleRules
	{
		public LocationServicesIOSImpl(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			PublicIncludePathModuleNames.AddRange
			(
				new string[]
				{
					"LocationServicesBPLibrary",
					"LocationServicesIOSImpl"
				}
			);
					
			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Launch",
					"LocationServicesBPLibrary"
					// ... add private dependencies that you statically link with here ...
				}
			);
			
			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"LocationServicesBPLibrary",
					// ... add private dependencies that you statically link with here ...
				}
			);

			
			// Additional Frameworks and Libraries for IOS
			if (Target.Platform == UnrealTargetPlatform.IOS)
			{
				PublicFrameworks.AddRange
				(
					new string[]
					{
						"CoreLocation",
					}
				);
			}
        }
	}
}
