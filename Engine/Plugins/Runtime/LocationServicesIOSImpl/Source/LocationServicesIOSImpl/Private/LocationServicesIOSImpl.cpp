// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocationServicesIOSImpl.h"
#include <CoreLocation/CoreLocation.h>
#import <UIKit/UIKit.h>

DEFINE_LOG_CATEGORY(LogLocationServicesIOS);

/**
* FLocationManagerDelegate allows us to get callbacks from the LocationManager when there is an update to
* the user's location.
*/
@interface FLocationManagerDelegate : UIViewController<CLLocationManagerDelegate>

-(bool)initLocationServices: (CLLocationAccuracy) accuracy distanceFilterValue:(CLLocationDistance) distanceFilter;
-(void)startUpdatingLocation;
-(void)stopUpdatingLocation;
-(CLLocation*)getLastKnownLocation;
-(void)dealloc;

@property CLLocationAccuracy Accuracy;
@property CLLocationDistance DistanceFilter;
@property(strong, nonatomic) CLLocationManager* LocManager;

@end

@implementation FLocationManagerDelegate
@synthesize Accuracy;
@synthesize DistanceFilter;
@synthesize LocManager;

-(bool)initLocationServices:(CLLocationAccuracy) accuracy distanceFilterValue:(CLLocationDistance) distanceFilter
{
    Accuracy = accuracy;
    DistanceFilter = distanceFilter;
    
    return true;
}
-(void)startUpdatingLocation
{
     LocManager = [[CLLocationManager alloc] init];
     
     LocManager.delegate = self;
     LocManager.distanceFilter = DistanceFilter;
     LocManager.desiredAccuracy = Accuracy;
    
     if (LocManager.authorizationStatus == kCLAuthorizationStatusNotDetermined)
     {
        //we need to request location services
        [LocManager requestAlwaysAuthorization];
     }
     
     if ([CLLocationManager locationServicesEnabled])
     {
        [LocManager startUpdatingLocation];
     }   
}
-(void)stopUpdatingLocation
{
    [LocManager stopUpdatingLocation];
    
    [LocManager release];
    LocManager = nil;
}
-(CLLocation*)getLastKnownLocation
{
    if(LocManager != nil)
    {
        return LocManager.location;
    }
    
    return nil;
}
-(void)dealloc
{
	if (LocManager != nil)
	{
		LocManager.delegate = nil;
		[LocManager release];
		LocManager = nil;
	}
	[super dealloc];
}
/*
* Callback from the LocationManager when there's an update to our location
*/
- (void)locationManager:(CLLocationManager *)manager didUpdateLocations:(NSArray<CLLocation *> *)locations
{
    CLLocation *newLocation = [locations lastObject];
    CLLocation *oldLocation;
    if (locations.count > 1) {
        oldLocation = [locations objectAtIndex:locations.count-2];
    } else {
        oldLocation = nil;
    }
    
	FLocationServicesData locationData;
    locationData.Timestamp = (float)newLocation.timestamp.timeIntervalSince1970;
    locationData.Longitude = (float)newLocation.coordinate.longitude;
    locationData.Latitude = (float)newLocation.coordinate.latitude;
    locationData.HorizontalAccuracy = (float)newLocation.horizontalAccuracy;
    locationData.VerticalAccuracy = (float)newLocation.verticalAccuracy;
    locationData.Altitude = (float)newLocation.altitude;
    
    ULocationServices::GetLocationServicesImpl()->OnLocationChanged.Broadcast(locationData);
}

/*
* Callback from the LocationManager when there's an error with location services
*/
- (void)locationManager:(CLLocationManager *)manager didFailWithError:(NSError *)error
{
    UE_LOGF(LogLocationServicesIOS, Log, "IOS locationManager didFailWithError");
        
    NSLog(@"%@",[error localizedDescription]);

    if([[error domain] isEqualToString:@"kCLErrorDomain"] && [error code] == 0)
    {
        //this is a common error when the location service could not retrieve the user's location, suggested
        //fix is to restart the service
        [self stopUpdatingLocation];
        [self startUpdatingLocation];
    }
}
@end

ULocationServicesIOSImpl::~ULocationServicesIOSImpl()
{

}

bool ULocationServicesIOSImpl::InitLocationServices(ELocationAccuracy Accuracy, float UpdateFrequency, float MinDistance)
{
	//initialize the location manager with our settings
    FLocationManagerDelegate* locDelegate = [[FLocationManagerDelegate alloc] init];
    LocationDelegate = [locDelegate retain];
    
	//CLLocationDistance is just a double, convert before calling the delegate function
	CLLocationDistance distanceFilter = (CLLocationDistance)MinDistance;

	CLLocationAccuracy locationAccuracy;

	switch (Accuracy)
	{
	case ELocationAccuracy::LA_ThreeKilometers:
		locationAccuracy = kCLLocationAccuracyThreeKilometers;
		break;
	case ELocationAccuracy::LA_OneKilometer:
		locationAccuracy = kCLLocationAccuracyKilometer;
		break;
	case ELocationAccuracy::LA_TenMeters:
		locationAccuracy = kCLLocationAccuracyNearestTenMeters;
		break;
	case ELocationAccuracy::LA_Best:
		locationAccuracy = kCLLocationAccuracyBest;
		break;
	case ELocationAccuracy::LA_Navigation:
		locationAccuracy = kCLLocationAccuracyBestForNavigation;
		break;
	default:
		locationAccuracy = kCLLocationAccuracyHundredMeters;
		break;
	}

    return [LocationDelegate initLocationServices:locationAccuracy distanceFilterValue:distanceFilter];
}

bool ULocationServicesIOSImpl::StartLocationService()
{
	if (LocationDelegate != nil)
	{
        dispatch_async(dispatch_get_main_queue(),^
        {
           [LocationDelegate startUpdatingLocation];
        });
        return true;
	}
	
	return false;
}


bool ULocationServicesIOSImpl::StopLocationService()
{
    if (LocationDelegate != nil)
    {
        dispatch_async(dispatch_get_main_queue(),^
        {
            [LocationDelegate stopUpdatingLocation];
        });
        return true;
    }
    
    return false;
}

FLocationServicesData ULocationServicesIOSImpl::GetLastKnownLocation()
{  
	FLocationServicesData LocData;

	if (LocationDelegate != nil)
	{
        CLLocation* CurrentLocation = [LocationDelegate getLastKnownLocation];
        if(CurrentLocation != nil)
        {           
            LocData.Timestamp = (float)CurrentLocation.timestamp.timeIntervalSince1970;
            LocData.Longitude = (float)CurrentLocation.coordinate.longitude;
            LocData.Latitude = (float)CurrentLocation.coordinate.latitude;
            LocData.HorizontalAccuracy = (float)CurrentLocation.horizontalAccuracy;
            LocData.VerticalAccuracy = (float)CurrentLocation.verticalAccuracy;
            LocData.Altitude = (float)CurrentLocation.altitude;
        }
	}

	return LocData;
}

bool ULocationServicesIOSImpl::IsLocationAccuracyAvailable(ELocationAccuracy Accuracy)
{
	//iOS should support any provided accuracy
	return true;
}
	
bool ULocationServicesIOSImpl::IsLocationServiceEnabled()
{
    CLLocationManager *LocManager = [[CLLocationManager alloc] init];;

	bool bEnabled = [CLLocationManager locationServicesEnabled];
	if (!bEnabled)
	{
		UE_LOGF(LogLocationServicesIOS, Log, "ULocationServicesIOSImpl::IsServiceEnabled() - location services disabled in settings");
	}
    bool bAuthorized = LocManager.authorizationStatus == kCLAuthorizationStatusDenied;
	if (!bAuthorized)
	{
		UE_LOGF(LogLocationServicesIOS, Log, "ULocationServicesIOSImpl::IsServiceEnabled() - location services have not been authorized for use");
	}

    [LocManager release];
    LocManager = nil;

	return bEnabled && bAuthorized;
}
