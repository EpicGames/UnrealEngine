// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocationServicesIOSImplModule.h"
#include "LocationServicesIOSImpl.h"
#include "LocationServicesBPLibrary.h"

IMPLEMENT_MODULE(FLocationServicesIOSImplModule, LocationServicesIOSImpl)

void FLocationServicesIOSImplModule::StartupModule()
{
	ImplInstance = NewObject<ULocationServicesIOSImpl>();
	ULocationServices::SetLocationServicesImpl(ImplInstance);
}

void FLocationServicesIOSImplModule::ShutdownModule()
{
	ULocationServices::ClearLocationServicesImpl();
	
	ImplInstance = NULL;
}